// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PLUGIN_NAME : ModuleRules
{
	public PLUGIN_NAME(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			[
				"Core",
			]
		);

		PrivateDependencyModuleNames.AddRange(
			[
				"AudioInsights",
				"AudioMixerCore",
				"CoreUObject",
				"InputCore",
				"Projects",
				"Slate",
				"SlateCore",
				"TraceAnalysis",
			]
		);
		
		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				[
					"AudioInsightsEditor",
					"Engine",
				]
			);
		}
	}
}
