// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTraceMessages.h"

#include "AudioInsightsConstants.h"
#include "Trace/TraceWriter.h"
#include "TraceWriter/CacheWriteHandler.h"

namespace PLUGIN_NAME
{
	using ::UE::Trace::IAnalyzer;
	using ::UE::Audio::Insights::FCacheWriteHandler;

	namespace ObjectMessageNames
	{
		extern const FName CreatedName = "Created";
		extern const FName ValueName = "Value";
		extern const FName DestroyedName = "Destroyed";
	};

	// FObjectMessageBase
	FObjectMessageBase::FObjectMessageBase(const IAnalyzer::FOnEventContext& InContext)
	{
		const IAnalyzer::FEventData& EventData = InContext.EventData;
	
		DeviceId = EventData.GetValue<uint32>("DeviceId");
		ID = EventData.GetValue<uint32>("ID");
		Timestamp = InContext.EventTime.AsSeconds(EventData.GetValue<uint64>(UE::Audio::Insights::TimestampFieldName));
	}
	
	// FObjectMessageCreatedMessage
	FObjectMessageCreatedMessage::FObjectMessageCreatedMessage(const IAnalyzer::FOnEventContext& InContext)
		: FObjectMessageBase(InContext)
	{
		const IAnalyzer::FEventData& EventData = InContext.EventData;

		EventData.GetString("Name", Name);
	}

	uint32 FObjectMessageCreatedMessage::GetSizeOf() const
	{
		return sizeof(FObjectMessageCreatedMessage);
	}

	FCacheWriteHandler FObjectMessageCreatedMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Object"), ANSITEXTVIEW("ObjectCreated"))
					.Field(ANSITEXTVIEW("DeviceId"),  UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("ID"),        UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"), UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("Name"),      UE::Trace::ETraceWriterFieldType::WideString)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FObjectMessageCreatedMessage& Msg = static_cast<const FObjectMessageCreatedMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",  Msg.DeviceId)
					.Field("ID",        Msg.ID)
					.Field("Timestamp", TimestampCycles)
					.Field("Name",      FStringView(Msg.Name))
					.End();
			}
		};
	}

	// FObjectMessageValueMessage
	FObjectMessageValueMessage::FObjectMessageValueMessage(const IAnalyzer::FOnEventContext& InContext)
		: FObjectMessageBase(InContext)
	{
		const IAnalyzer::FEventData& EventData = InContext.EventData;
	
		Value = EventData.GetValue<float>("Value");
	}
	
	uint32 FObjectMessageValueMessage::GetSizeOf() const
	{
		return sizeof(FObjectMessageValueMessage);
	}

	FCacheWriteHandler FObjectMessageValueMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Object"), ANSITEXTVIEW("ObjectValue"))
					.Field(ANSITEXTVIEW("DeviceId"),  UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("ID"),        UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"), UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("Value"),     UE::Trace::ETraceWriterFieldType::Float32)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FObjectMessageValueMessage& Msg = static_cast<const FObjectMessageValueMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",  Msg.DeviceId)
					.Field("ID",        Msg.ID)
					.Field("Timestamp", TimestampCycles)
					.Field("Value",     Msg.Value)
					.End();
			}
		};
	}
	
	// FObjectMessageDestroyedMessage
	FObjectMessageDestroyedMessage::FObjectMessageDestroyedMessage(const IAnalyzer::FOnEventContext& InContext)
		: FObjectMessageBase(InContext)
	{
	}
	
	uint32 FObjectMessageDestroyedMessage::GetSizeOf() const
	{
		return sizeof(FObjectMessageDestroyedMessage);
	}

	FCacheWriteHandler FObjectMessageDestroyedMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Object"), ANSITEXTVIEW("ObjectDestroyed"))
					.Field(ANSITEXTVIEW("DeviceId"),  UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("ID"),        UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"), UE::Trace::ETraceWriterFieldType::Uint64)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FObjectMessageDestroyedMessage& Msg = static_cast<const FObjectMessageDestroyedMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",  Msg.DeviceId)
					.Field("ID",        Msg.ID)
					.Field("Timestamp", TimestampCycles)
					.End();
			}
		};
	}

	// FObjectDashboardEntry
	FText FObjectDashboardEntry::GetDisplayName() const
	{
		const FString DisplayName = FSoftObjectPath(Name).GetAssetName();
		return DisplayName.IsEmpty() ? FText::FromString(Name) : FText::FromString(DisplayName);
	}
	
	const UObject* FObjectDashboardEntry::GetObject() const
	{
		return FSoftObjectPath(Name).ResolveObject();
	}
	
	UObject* FObjectDashboardEntry::GetObject()
	{
		return FSoftObjectPath(Name).ResolveObject();
	}
	
	bool FObjectDashboardEntry::IsValid() const
	{
		return ID != static_cast<uint32>(INDEX_NONE);
	}
} // namespace PLUGIN_NAME
