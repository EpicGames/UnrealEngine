// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Defines the trace message types for this dashboard.
 * Each message struct corresponds to a type of trace event received from the engine.
 * Messages are deserialized from trace data in ConstructAnalyzer (ObjectTraceProvider)
 * and queued for processing in ProcessMessages.
 *
 * The sending side (your system) emits events using UE_TRACE macros. For example,
 * the ObjectCreated event would be declared and sent like this:
 *
 *   // Declare the event schema (typically in a header or near the emitting code):
 *   UE_TRACE_EVENT_BEGIN(Object, ObjectCreated)
 *       UE_TRACE_EVENT_FIELD(uint32, DeviceId)
 *       UE_TRACE_EVENT_FIELD(uint32, ID)
 *       UE_TRACE_EVENT_FIELD(uint64, Timestamp)
 *       UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
 *   UE_TRACE_EVENT_END()
 *
 *   // Emit the event (in your system's runtime code):
 *   UE_TRACE_LOG(Object, ObjectCreated, AudioChannel)
 *       << ObjectCreated.DeviceId(InDeviceId)
 *       << ObjectCreated.ID(InObjectId)
 *       << ObjectCreated.Timestamp(FPlatformTime::Cycles64())
 *       << ObjectCreated.Name(*InObjectName);
 *
 * The channel/event names ("Object"/"ObjectCreated") must match the RouteEvent
 * calls in ConstructAnalyzer, and the field names must match the GetValue/GetString
 * calls in each message constructor below.
 *
 * To add a new message type:
 * 1. Define a new struct inheriting from FObjectMessageBase
 * 2. Implement GetMessageName() and GetSizeOf()
 * 3. Add a corresponding TAnalyzerMessageQueue in FObjectMessages
 * 4. Add a RouteId and case in ConstructAnalyzer's OnHandleEvent switch
 * 5. Process it in FObjectTraceProvider::ProcessMessages
 */

#pragma once

#include "Cache/IAudioCachedMessage.h"
#include "Messages/AnalyzerMessageQueue.h"
#include "Views/TableDashboardViewFactory.h"

namespace PLUGIN_NAME
{
	namespace ObjectMessageNames
	{
		extern const FName CreatedName;
		extern const FName ValueName;
		extern const FName DestroyedName;
		// You can add more names here as needed
	};

	struct FObjectMessageBase : UE::Audio::Insights::IAudioCachedMessage
	{
		FObjectMessageBase() = default;
		explicit FObjectMessageBase(const UE::Trace::IAnalyzer::FOnEventContext& InContext);

		virtual uint64 GetID() const override { return ID; }

		::Audio::FDeviceId DeviceId = INDEX_NONE;
		uint32 ID = INDEX_NONE;
	};
	
	/** Sent when an object is first created. Carries the Name used for display. */
	struct FObjectMessageCreatedMessage : public FObjectMessageBase
	{
		FObjectMessageCreatedMessage() = default;
		FObjectMessageCreatedMessage(const UE::Trace::IAnalyzer::FOnEventContext& InContext);

		virtual const FName GetMessageName() const override { return ObjectMessageNames::CreatedName; }
		virtual uint32 GetSizeOf() const override;
		virtual UE::Audio::Insights::FCacheWriteHandler GetCacheWriteHandler() const override;

		FString Name;
	};

	/** Sent periodically with updated data for an existing object. */
	struct FObjectMessageValueMessage : public FObjectMessageBase
	{
		FObjectMessageValueMessage() = default;
		FObjectMessageValueMessage(const UE::Trace::IAnalyzer::FOnEventContext& InContext);

		virtual const FName GetMessageName() const override { return ObjectMessageNames::ValueName; }
		virtual uint32 GetSizeOf() const override;
		
		/**
		 * Opt-in to have this message included in Audio Insights cache snapshots (.utrace export).
		 * Implementing this override enables the message to be serialized when saving an Audio Insights
		 * session to disk via the save cache snapshot button. The handler declares the event schema (field names and types) and writes
		 * each message's data to the trace stream. If not overridden, the base class returns an empty
		 * handler and the message type is excluded from cache snapshots.
		 */
		virtual UE::Audio::Insights::FCacheWriteHandler GetCacheWriteHandler() const override;

		float Value = 0.0f;
	};

	/** Sent when an object is destroyed. Only carries the base fields (DeviceId, ID, Timestamp). */
	struct FObjectMessageDestroyedMessage : public FObjectMessageBase
	{
		FObjectMessageDestroyedMessage() = default;
		FObjectMessageDestroyedMessage(const UE::Trace::IAnalyzer::FOnEventContext& InContext);

		virtual const FName GetMessageName() const override { return ObjectMessageNames::DestroyedName; }
		virtual uint32 GetSizeOf() const override;
		virtual UE::Audio::Insights::FCacheWriteHandler GetCacheWriteHandler() const override;
	};

	// You can add more structs that inherit from FObjectMessageBase here

	/** Thread-safe message queues that buffer incoming trace messages for processing by the provider. */
	class FObjectMessages
	{
		UE::Audio::Insights::TAnalyzerMessageQueue<FObjectMessageCreatedMessage> CreatedMessages;
		UE::Audio::Insights::TAnalyzerMessageQueue<FObjectMessageValueMessage> ValueMessages;
		UE::Audio::Insights::TAnalyzerMessageQueue<FObjectMessageDestroyedMessage> DestroyedMessages;

		friend class FObjectTraceProvider;
	};
	
	/** Data displayed per row in the dashboard table. Populated by the provider from trace messages. */
	struct FObjectDashboardEntry : UE::Audio::Insights::IObjectDashboardEntry
	{
		FObjectDashboardEntry() = default;
		virtual ~FObjectDashboardEntry() = default;
		
		virtual bool IsValid() const override;
		
		virtual FText GetDisplayName() const override;
		virtual const UObject* GetObject() const override;
		virtual UObject* GetObject() override;
	
		FString Name;
	
		::Audio::FDeviceId DeviceId = INDEX_NONE;
		uint32 ID = INDEX_NONE;
		
		double Timestamp = 0.0;
		
		float Value = 0.0f;
	};
} // namespace PLUGIN_NAME
