// Copyright Epic Games, Inc. All Rights Reserved.
#include "PLUGIN_NAMEModule.h"

#include "IAudioInsightsModule.h"
#include "Views/ObjectDashboardViewFactory.h"

#if WITH_EDITOR
#include "IAudioInsightsEditorModule.h"
#endif // WITH_EDITOR

void FPLUGIN_NAMEModule::StartupModule()
{
	if (!IsRunningCommandlet())
	{
#if WITH_EDITOR
		IAudioInsightsEditorModule& AudioInsightsModule = IAudioInsightsEditorModule::GetChecked();
#else
		IAudioInsightsModule& AudioInsightsModule = IAudioInsightsModule::GetChecked();
#endif // WITH_EDITOR
		
		AudioInsightsModule.RegisterDashboardViewFactory(MakeShared<PLUGIN_NAME::FObjectDashboardViewFactory>());
	}
}

void FPLUGIN_NAMEModule::ShutdownModule()
{
	if (!IsRunningCommandlet())
	{
#if WITH_EDITOR
		if (!IAudioInsightsEditorModule::IsModuleLoaded())
		{
			return;
		}
	
		IAudioInsightsEditorModule& AudioInsightsModule = IAudioInsightsEditorModule::GetChecked();
#else
		if (!IAudioInsightsModule::IsModuleLoaded())
		{
			return;
		}
	
		IAudioInsightsModule& AudioInsightsModule = IAudioInsightsModule::GetChecked();
#endif // WITH_EDITOR
		
		AudioInsightsModule.UnregisterDashboardViewFactory(PLUGIN_NAME::ObjectDashboardViewFactoryName);
	}
}

IMPLEMENT_MODULE(FPLUGIN_NAMEModule, PLUGIN_NAME);
