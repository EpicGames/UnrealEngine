// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

#define UE_API PLUGIN_NAME_API

/**
 * Module entry point for this Audio Insights plugin.
 * StartupModule registers the dashboard view factory with Audio Insights,
 * which creates the tab in the Audio Insights dashboard.
 * ShutdownModule unregisters it on plugin teardown.
 */
class FPLUGIN_NAMEModule : public IModuleInterface
{
public:
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
};

#undef UE_API
