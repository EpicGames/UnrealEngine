// Copyright Epic Games, Inc. All Rights Reserved.

#include "PLUGIN_NAMEStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

#define LOCTEXT_NAMESPACE "PLUGIN_NAME"

namespace PLUGIN_NAME
{
	FStyle::FStyle()
		: FSlateStyleSet(GetStyleName())
	{
		SetParentStyleName(FAppStyle::GetAppStyleSetName());

		// Register style content paths
		const TSharedPtr<const IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("PLUGIN_NAME"));
		if (Plugin.IsValid())
		{
			SetContentRoot(Plugin->GetBaseDir() / "Resources");
		}
		SetCoreContentRoot(FPaths::EngineContentDir() / "Slate");

		// Icons
		const FVector2D Icon16(16.0f, 16.0f);

		// Add your custom icons here
		// Set("ICON_NAME", new IMAGE_BRUSH(TEXT("ICON_NAME"), Icon16));

		// Register slate style
		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FStyle::~FStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	const FStyle& FStyle::Get()
	{
		static FStyle Style;
		return Style;
	}

	const FName& FStyle::GetStyleName()
	{
		static const FName StyleName = "PLUGIN_NAMEStyle";
		return StyleName;
	}

	FSlateIcon FStyle::CreateIcon(const FName& InName) const
	{
		return { GetStyleName(), InName };
	}
} // namespace PLUGIN_NAME

#undef LOCTEXT_NAMESPACE
