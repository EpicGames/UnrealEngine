// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "Styling/SlateStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
 
namespace PLUGIN_NAME
{
	/**
	 * Slate style set for this plugin's custom icons and visual elements.
	 * Register custom brushes and icons in the FStyle constructor (PLUGIN_NAMEStyle.cpp)
	 * and reference them by name throughout the plugin.
	 */
	class FStyle final : public FSlateStyleSet
	{
	public:
		FStyle();
		~FStyle();

		static const FStyle& Get();
		static const FName& GetStyleName();
	
		FSlateIcon CreateIcon(const FName& InName) const;
	};
}
