// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTraceProvider.h"

#include "AudioInsightsConstants.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "IAudioInsightsModule.h"

namespace PLUGIN_NAME
{
	using ::UE::Trace::IAnalyzer;
	
	FObjectTraceProvider::FObjectTraceProvider()
		: UE::Audio::Insights::TDeviceDataMapTraceProvider<uint32, TSharedPtr<FObjectDashboardEntry>>(GetName_Static())
	{
	}

	FName FObjectTraceProvider::GetName_Static()
	{
		static const FLazyName TraceProviderName = "ObjectProvider";
		return TraceProviderName;
	}

	IAnalyzer* FObjectTraceProvider::ConstructAnalyzer(TraceServices::IAnalysisSession& InSession)
	{
		class FTraceAnalyzer : public FTraceAnalyzerBase
		{
		public:
			FTraceAnalyzer(TSharedRef<FObjectTraceProvider> InProvider, TraceServices::IAnalysisSession& InSession)
				: FTraceAnalyzerBase(InProvider)
				, Session(InSession)
			{
			}

			/** Registers trace event routes that map channel/event name pairs to RouteIds. */
			virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override
			{
				FTraceAnalyzerBase::OnAnalysisBegin(Context);

				UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;

				Builder.RouteEvent(RouteId_Created,   "Object", "ObjectCreated");
				Builder.RouteEvent(RouteId_Value,     "Object", "ObjectValue");
				Builder.RouteEvent(RouteId_Destroyed, "Object", "ObjectDestroyed");
			}

			/** Dispatches incoming trace events to the appropriate message queue based on RouteId. */
			virtual bool OnHandleEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override
			{
				LLM_SCOPE_BYNAME(TEXT("Insights/FTraceAnalyzer"));

				FObjectMessages& Messages = GetProvider<FObjectTraceProvider>().TraceMessages;

				switch (RouteId)
				{
					case RouteId_Created:
					{
						CacheMessage<FObjectMessageCreatedMessage>(Context, Messages.CreatedMessages);
						break;
					}
					case RouteId_Value:
					{
						CacheMessage<FObjectMessageValueMessage>(Context, Messages.ValueMessages);
						break;
					}
					case RouteId_Destroyed:
					{
						CacheMessage<FObjectMessageDestroyedMessage>(Context, Messages.DestroyedMessages);
						break;
					}
					default:
					{
						return OnEventFailure(RouteId, Style, Context);
					}
				}

				const double Timestamp = Context.EventTime.AsSeconds(Context.EventData.GetValue<uint64>(UE::Audio::Insights::TimestampFieldName));

				{
					TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
					Session.UpdateDurationSeconds(Timestamp);
				}

				return OnEventSuccess(RouteId, Style, Context);
			}

		private:
			enum : uint16
			{
				RouteId_Created,
				RouteId_Value,
				RouteId_Destroyed
			};

			TraceServices::IAnalysisSession& Session;
		};

		// AsShared() assumes this provider is owned by a TSharedPtr/TSharedRef — the dashboard factory
		// creates it via MakeShared<FObjectTraceProvider>(). If that ownership strategy changes,
		// AsShared() will assert.
		return new FTraceAnalyzer(AsShared(), InSession);
	}

	bool FObjectTraceProvider::ProcessMessages()
	{
		// Helper lambdas
		auto CreateEntryFunc = [this](const FObjectMessageBase& Msg)
		{
			TSharedPtr<FObjectDashboardEntry>* ToReturn = nullptr;

			UpdateDeviceEntry(Msg.DeviceId, Msg.ID, [&ToReturn, &Msg](TSharedPtr<FObjectDashboardEntry>& Entry)
			{
				if (!Entry.IsValid())
				{
					Entry = MakeShared<FObjectDashboardEntry>();

					Entry->DeviceId = Msg.DeviceId;
					Entry->ID = Msg.ID;
				}

				Entry->Timestamp = Msg.Timestamp;

				ToReturn = &Entry;
			});

			return ToReturn;
		};

		auto GetEntryFunc = [this](const FObjectMessageBase& Msg)
		{
			return FindDeviceEntry(Msg.DeviceId, Msg.ID);
		};

		// Process Created messages first - populates the entry Name (one-time)
		ProcessMessageQueue<FObjectMessageCreatedMessage>(TraceMessages.CreatedMessages, CreateEntryFunc,
		[this](const FObjectMessageCreatedMessage& Msg, TSharedPtr<FObjectDashboardEntry>* OutEntry)
		{
			if (OutEntry != nullptr)
			{
				FObjectDashboardEntry& EntryRef = *OutEntry->Get();
				EntryRef.Name = Msg.Name;
			}
		});

		// Process Value messages - updates the entry's numeric value
		ProcessMessageQueue<FObjectMessageValueMessage>(TraceMessages.ValueMessages, GetEntryFunc,
		[this](const FObjectMessageValueMessage& Msg, TSharedPtr<FObjectDashboardEntry>* OutEntry)
		{
			if (OutEntry != nullptr)
			{
				FObjectDashboardEntry& EntryRef = *OutEntry->Get();
				EntryRef.Value = Msg.Value;
			}
		});

		// Process Destroyed messages last - removes the entry from the data map
		ProcessMessageQueue<FObjectMessageDestroyedMessage>(TraceMessages.DestroyedMessages, GetEntryFunc,
		[this](const FObjectMessageDestroyedMessage& Msg, TSharedPtr<FObjectDashboardEntry>* OutEntry)
		{
			if (OutEntry != nullptr)
			{
				RemoveDeviceEntry(Msg.DeviceId, Msg.ID);
			}
		});

		return true;
	}
	
	void FObjectTraceProvider::OnTimingViewTimeMarkerChanged(double TimeMarker)
	{
		using namespace ::UE::Audio::Insights;

		DeviceDataMap.Empty();

		const FAudioInsightsCacheManager& CacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();

		// Replay Created messages to reconstruct entries with their names
		CacheManager.IterateTo<FObjectMessageCreatedMessage>(ObjectMessageNames::CreatedName, TimeMarker,
		[this](const FObjectMessageCreatedMessage& Msg)
		{
			UpdateDeviceEntry(Msg.DeviceId, Msg.ID, [&Msg](TSharedPtr<FObjectDashboardEntry>& Entry)
			{
				if (!Entry.IsValid())
				{
					Entry = MakeShared<FObjectDashboardEntry>();
					Entry->DeviceId = Msg.DeviceId;
					Entry->ID = Msg.ID;
				}

				Entry->Timestamp = Msg.Timestamp;
				Entry->Name = Msg.Name;
			});
		});

		// Replay Destroyed messages to remove entries that were destroyed before TimeMarker
		CacheManager.IterateTo<FObjectMessageDestroyedMessage>(ObjectMessageNames::DestroyedName, TimeMarker,
		[this](const FObjectMessageDestroyedMessage& Msg)
		{
			auto* OutEntry = FindDeviceEntry(Msg.DeviceId, Msg.ID);

			if (OutEntry != nullptr && (*OutEntry)->Timestamp < Msg.Timestamp)
			{
				RemoveDeviceEntry(Msg.DeviceId, Msg.ID);
			}
		});

		// For each surviving entry, find the closest Value message at the given timestamp.
		// If the data is dense, consider using ParallelFor here for better performance.
		const FDeviceData* DeviceData = FindFilteredDeviceData();
		
		if (DeviceData != nullptr)
		{
			for (auto& [ID, Entry] : *DeviceData)
			{
				const FObjectMessageValueMessage* FoundMessage = CacheManager.FindClosestMessage<FObjectMessageValueMessage>(
					ObjectMessageNames::ValueName, TimeMarker, ID);

				if (FoundMessage != nullptr)
				{
					Entry->Timestamp = FoundMessage->Timestamp;
					Entry->Value = FoundMessage->Value;
				}
			}
		}

		// Call parent method to update LastMessageId
		FTraceProviderBase::OnTimingViewTimeMarkerChanged(TimeMarker);
	}
} // namespace PLUGIN_NAME
