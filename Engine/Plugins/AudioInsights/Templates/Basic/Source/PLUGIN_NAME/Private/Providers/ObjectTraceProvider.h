// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioInsightsTraceProviderBase.h"
#include "Messages/ObjectTraceMessages.h"

namespace PLUGIN_NAME
{
	/**
	 * Trace provider for the Object dashboard.
	 * A trace provider is responsible for:
	 * 1. Constructing an analyzer (ConstructAnalyzer) that deserializes trace events
	 *    and queues them as typed messages.
	 * 2. Processing queued messages (ProcessMessages) to create/update dashboard entries
	 *    that the dashboard view factory displays.
	 * 3. Responding to timeline scrubbing (OnTimingViewTimeMarkerChanged) to reconstruct
	 *    dashboard state at a given point in time.
	 */
	class FObjectTraceProvider
		: public UE::Audio::Insights::TDeviceDataMapTraceProvider<uint32, TSharedPtr<FObjectDashboardEntry>>
		, public TSharedFromThis<FObjectTraceProvider>
	{
	public:
		FObjectTraceProvider();
		virtual ~FObjectTraceProvider() = default;

		/**
		 * Constructs a trace analyzer that runs on the trace analysis thread.
		 * The analyzer is defined as an inner class because each provider needs
		 * its own set of route IDs and message-type handling logic.
		 * OnAnalysisBegin registers trace event routes; OnHandleEvent dispatches
		 * incoming events to the appropriate message queue via CacheMessage.
		 */
		virtual UE::Trace::IAnalyzer* ConstructAnalyzer(TraceServices::IAnalysisSession& InSession) override;

		/**
		 * All providers should have a GetName_Static call so that FTraceDashboardViewFactoryBase::FindProvider can use it.
		 */
		static FName GetName_Static();

	private:
		/**
		 * Called on the game thread via FTSTicker to drain message queues and update dashboard entries.
		 * Uses a two-step pattern per message type:
		 * 1. A "get or create entry" lambda that finds or creates the dashboard entry for a message's ID
		 * 2. A processing lambda that applies message-specific data to that entry
		 * ProcessMessageQueue (from TDeviceDataMapTraceProvider) handles thread-safe dequeue
		 * and invokes both lambdas for each message.
		 */
		virtual bool ProcessMessages() override;

		/**
		 * Called when the user scrubs the timeline in the Timing Insights view.
		 * Reconstructs dashboard state at the given TimeMarker by replaying cached
		 * Created/Destroyed messages, then finding the closest Value per survivor.
		 */
		virtual void OnTimingViewTimeMarkerChanged(double TimeMarker) override;
	
		FObjectMessages TraceMessages;
	};
} // namespace PLUGIN_NAME