// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectDashboardViewFactory.h"

#include "AudioInsightsStyle.h"
#include "IAudioInsightsModule.h"
#include "PLUGIN_NAMEStyle.h"
#include "Providers/ObjectTraceProvider.h"
#include "Templates/SharedPointer.h"

#define LOCTEXT_NAMESPACE "PLUGIN_NAME"

namespace PLUGIN_NAME
{
	using namespace ::UE::Audio::Insights;

	namespace FObjectDashboardViewFactoryPrivate
	{
		const FName NameColumnName  = "Name";
		const FName ValueColumnName = "Value";

		const FObjectDashboardEntry& CastEntry(const IDashboardDataViewEntry& InData)
		{
			return static_cast<const FObjectDashboardEntry&>(InData);
		};
	} // namespace FObjectDashboardViewFactoryPrivate

	const FLazyName ObjectDashboardViewFactoryName("Object");

	FObjectDashboardViewFactory::FObjectDashboardViewFactory()
	{
		IAudioInsightsTraceModule& InsightsTraceModule = IAudioInsightsModule::GetChecked().GetTraceModule();

		TSharedPtr<FObjectTraceProvider> Provider = MakeShared<FObjectTraceProvider>();
		InsightsTraceModule.AddTraceProvider(Provider);
	
		Providers =
		{
			StaticCastSharedPtr<FTraceProviderBase>(Provider)
		};

		SortByColumn = FObjectDashboardViewFactoryPrivate::NameColumnName;
		SortMode     = EColumnSortMode::Ascending;
	}
	
	FName FObjectDashboardViewFactory::GetName() const
	{
		return PLUGIN_NAME::ObjectDashboardViewFactoryName;
	}
	
	FText FObjectDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("Object_Name", "Object");
	}
	
	FSlateIcon FObjectDashboardViewFactory::GetIcon() const
	{
		// Icon should be defined in the FStyle and its name referenced here, returning a default one for now
		//return PLUGIN_NAME::FStyle::Get().CreateIcon("AudioInsightsTemplate.Icon.IconName"); 
		return UE::Audio::Insights::FSlateStyle::Get().CreateIcon("AudioInsights.Icon.Dashboard");
	}
	
	EDefaultDashboardTabStack FObjectDashboardViewFactory::GetDefaultTabStack() const
	{
		return EDefaultDashboardTabStack::Log;
	}
	
	TSharedRef<SWidget> FObjectDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		return FTraceTableDashboardViewFactory::MakeWidget(OwnerTab, SpawnTabArgs);
	}
	
	const TMap<FName, FTraceTableDashboardViewFactory::FColumnData>& FObjectDashboardViewFactory::GetColumns() const
	{
		using namespace FObjectDashboardViewFactoryPrivate;
		
		auto CreateColumnData = []()
		{
			return TMap<FName, FTraceTableDashboardViewFactory::FColumnData>
			{
				{
					NameColumnName,
					{
						LOCTEXT("ObjectDashboard_NameColumnDisplayName", "Name"),
						[](const IDashboardDataViewEntry& InData)
						{
							return CastEntry(InData).GetDisplayName();
						},
						nullptr /*GetIconName*/,
						false /* bDefaultHidden */,
						0.85f /* FillWidth */
					}
				},
				{
					ValueColumnName,
					{
						LOCTEXT("ObjectDashboard_ValueColumnDisplayName", "Value"),
						[](const IDashboardDataViewEntry& InData)
						{
							const FObjectDashboardEntry& ObjectDashboardEntry = CastEntry(InData);
							return FText::AsNumber(ObjectDashboardEntry.Value);
						},
						nullptr /*GetIconName*/,
						false /* bDefaultHidden */,
						0.15f /* FillWidth */
					}
				}
			};
		};
		
		static const TMap<FName, FTraceTableDashboardViewFactory::FColumnData> ColumnData = CreateColumnData();
		
		return ColumnData;
	}
	
	void FObjectDashboardViewFactory::ProcessEntries(EProcessReason Reason)
	{
		const FString FilterString = GetSearchFilterText().ToString();
			
		FilterEntries<FObjectTraceProvider>([&FilterString](const IDashboardDataViewEntry& Entry)
		{
			const FObjectDashboardEntry& DashboardEntry = static_cast<const FObjectDashboardEntry&>(Entry);
			return DashboardEntry.GetDisplayName().ToString().Contains(FilterString);
		});
	}
	
	void FObjectDashboardViewFactory::SortTable()
	{
		using namespace FObjectDashboardViewFactoryPrivate;

		if (SortByColumn == NameColumnName)
		{
			SortByPredicate<FObjectDashboardEntry>([](const FObjectDashboardEntry& First, const FObjectDashboardEntry& Second)
			{
				const int32 NameComparison = First.GetDisplayName().CompareToCaseIgnored(Second.GetDisplayName());

				if (NameComparison != 0)
				{
					return NameComparison < 0;
				}

				return First.ID < Second.ID;
			});
		}
		else if (SortByColumn == ValueColumnName)
		{
			SortByPredicate<FObjectDashboardEntry>([](const FObjectDashboardEntry& First, const FObjectDashboardEntry& Second)
			{
				if (!FMath::IsNearlyEqual(First.Value, Second.Value, UE_KINDA_SMALL_NUMBER))
				{
					return First.Value < Second.Value;
				}

				return First.ID < Second.ID;
			});
		}
	}
} // namespace PLUGIN_NAME

#undef LOCTEXT_NAMESPACE
 