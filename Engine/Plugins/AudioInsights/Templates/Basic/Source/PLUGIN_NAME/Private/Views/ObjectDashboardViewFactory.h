// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Views/TableDashboardViewFactory.h"

namespace PLUGIN_NAME
{
	extern const FLazyName ObjectDashboardViewFactoryName;

	/**
	 * Dashboard view factory for the Object dashboard tab.
	 * This factory defines what appears in the Audio Insights UI for this tab:
	 * - GetName/GetDisplayName: tab identification and display text
	 * - GetIcon: tab icon (uses AudioInsights default; register your own in FStyle)
	 * - GetDefaultTabStack: which tab group this dashboard belongs to (Log, Analysis, etc.)
	 * - GetColumns: defines the data columns shown in the table
	 * - ProcessEntries: filters entries based on the search bar text
	 * - SortTable: defines sort order when a column header is clicked
	 *
	 * The constructor creates the trace provider and registers it with Audio Insights.
	 * The base class handles the Slate widget creation and tick-driven data refresh.
	 *
	 * The new dashboard tab will be available in Audio Insights View -> GetDisplayName().
	 */
	class FObjectDashboardViewFactory : public UE::Audio::Insights::FTraceObjectTableDashboardViewFactory
	{
	public:
		FObjectDashboardViewFactory();
		virtual ~FObjectDashboardViewFactory() = default;

		virtual FName GetName() const override;
		virtual FText GetDisplayName() const override;
		virtual FSlateIcon GetIcon() const override;
		virtual UE::Audio::Insights::EDefaultDashboardTabStack GetDefaultTabStack() const override;

	private:
		virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;
		
		virtual const TMap<FName, FColumnData>& GetColumns() const override;
		virtual void ProcessEntries(EProcessReason Reason) override;
		virtual void SortTable() override;
	};
} // namespace PLUGIN_NAME
