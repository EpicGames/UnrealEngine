// Copyright Epic Games, Inc. All Rights Reserved.

#include "PLUGIN_NAMECommands.h"

#define LOCTEXT_NAMESPACE "FPLUGIN_NAMEModule"

void FPLUGIN_NAMECommands::RegisterCommands()
{
	UI_COMMAND(PluginAction, "PLUGIN_NAME", "Execute PLUGIN_NAME action", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
