// Copyright Epic Games, Inc. All Rights Reserved.

#include "PLUGIN_NAMEStyle.h"
#include "PLUGIN_NAME.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/SlateStyleRegistry.h"
#include "Slate/SlateGameResources.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleMacros.h"

#define RootToContentDir Style->RootToContentDir

TSharedPtr<FSlateStyleSet> FPLUGIN_NAMEStyle::StyleInstance = nullptr;

void FPLUGIN_NAMEStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FPLUGIN_NAMEStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FPLUGIN_NAMEStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("PLUGIN_NAMEStyle"));
	return StyleSetName;
}


const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);

TSharedRef< FSlateStyleSet > FPLUGIN_NAMEStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("PLUGIN_NAMEStyle"));
	Style->SetContentRoot(IPluginManager::Get().FindPlugin("PLUGIN_NAME")->GetBaseDir() / TEXT("Resources"));

	Style->Set("PLUGIN_NAME.PluginAction", new IMAGE_BRUSH_SVG(TEXT("PlaceholderButtonIcon"), Icon20x20));
	return Style;
}

void FPLUGIN_NAMEStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	}
}

const ISlateStyle& FPLUGIN_NAMEStyle::Get()
{
	return *StyleInstance;
}
